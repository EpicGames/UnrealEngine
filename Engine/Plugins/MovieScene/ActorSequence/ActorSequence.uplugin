{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Actor Sequence (Experimental)",
	"Description" : "Runtime for embedded actor sequences",
	"Category" : "Editor",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "ActorSequence",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault"
		},
		{
			"Name" : "ActorSequenceEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		}
	]
}