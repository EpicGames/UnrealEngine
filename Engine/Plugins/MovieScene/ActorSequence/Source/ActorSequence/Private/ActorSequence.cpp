// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorSequence.h"
#include "Engine/Level.h"
#include "IMovieScenePlayer.h"
#include "MovieScene.h"
#include "Modules/ModuleManager.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Blueprint.h"
#include "ActorSequenceComponent.h"
#include "Engine/LevelScriptActor.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Tracks/MovieSceneEventTrack.h"
#include "Tracks/MovieSceneMaterialParameterCollectionTrack.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "Tracks/MovieSceneTimeWarpTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorSequence)

IMPLEMENT_MODULE(FDefaultModuleImpl, ActorSequence);

#if WITH_EDITOR
UActorSequence::FOnInitialize UActorSequence::OnInitializeSequenceEvent;
#endif

static TAutoConsoleVariable<int32> CVarDefaultEvaluationType(
	TEXT("ActorSequence.DefaultEvaluationType"),
	0,
	TEXT("0: Playback locked to playback frames\n1: Unlocked playback with sub frame interpolation"),
	ECVF_Default);

static TAutoConsoleVariable<FString> CVarDefaultTickResolution(
	TEXT("ActorSequence.DefaultTickResolution"),
	TEXT("24000fps"),
	TEXT("Specifies default a tick resolution for newly created level sequences. Examples: 30 fps, 120/1 (120 fps), 30000/1001 (29.97), 0.01s (10ms)."),
	ECVF_Default);

static TAutoConsoleVariable<FString> CVarDefaultDisplayRate(
	TEXT("ActorSequence.DefaultDisplayRate"),
	TEXT("30fps"),
	TEXT("Specifies default a display frame rate for newly created level sequences; also defines frame locked frame rate where sequences are set to be frame locked. Examples: 30 fps, 120/1 (120 fps), 30000/1001 (29.97), 0.01s (10ms)."),
	ECVF_Default);

UActorSequence::UActorSequence(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MovieScene(nullptr)
#if WITH_EDITORONLY_DATA
	, bHasBeenInitialized(false)
#endif
{
	bParentContextsAreSignificant = true;

	MovieScene = ObjectInitializer.CreateDefaultSubobject<UMovieScene>(this, "MovieScene");
	MovieScene->SetFlags(RF_Transactional);
}

bool UActorSequence::IsEditable() const
{
	UObject* Template = GetArchetype();

	if (Template == GetDefault<UActorSequence>())
	{
		return false;
	}

	return !Template || Template->GetTypedOuter<UActorSequenceComponent>() == GetDefault<UActorSequenceComponent>();
}

#if WITH_EDITORONLY_DATA
UBlueprint* UActorSequence::GetParentBlueprint() const
{
	if (UBlueprintGeneratedClass* GeneratedClass = GetTypedOuter<UBlueprintGeneratedClass>())
	{
		return Cast<UBlueprint>(GeneratedClass->ClassGeneratedBy);
	}
	return nullptr;
}
#endif

void UActorSequence::PostInitProperties()
{
#if WITH_EDITOR && WITH_EDITORONLY_DATA

	// We do not run the default initialization for actor sequences that are CDOs, or that are going to be loaded (since they will have already been initialized in that case)
	EObjectFlags ExcludeFlags = RF_ClassDefaultObject | RF_NeedLoad | RF_NeedPostLoad | RF_NeedPostLoadSubobjects | RF_WasLoaded;

	UActorComponent* OwnerComponent = Cast<UActorComponent>(GetOuter());
	if (!bHasBeenInitialized && !HasAnyFlags(ExcludeFlags) && OwnerComponent && !OwnerComponent->HasAnyFlags(ExcludeFlags))
	{
		AActor* Actor = Cast<AActor>(OwnerComponent->GetOuter());

		FGuid BindingID = MovieScene->AddPossessable(Actor ? Actor->GetActorLabel() : TEXT("Owner"), Actor ? Actor->GetClass() : AActor::StaticClass());
		ObjectReferences.CreateBinding(BindingID, FActorSequenceObjectReference::CreateForContextActor());

		const bool bFrameLocked = CVarDefaultEvaluationType.GetValueOnGameThread() != 0;

		MovieScene->SetEvaluationType( bFrameLocked ? EMovieSceneEvaluationType::FrameLocked : EMovieSceneEvaluationType::WithSubFrames );

		FFrameRate TickResolution(60000, 1);
		TryParseString(TickResolution, *CVarDefaultTickResolution.GetValueOnGameThread());
		MovieScene->SetTickResolutionDirectly(TickResolution);

		FFrameRate DisplayRate(30, 1);
		TryParseString(DisplayRate, *CVarDefaultDisplayRate.GetValueOnGameThread());
		MovieScene->SetDisplayRate(DisplayRate);

		OnInitializeSequenceEvent.Broadcast(this);
		bHasBeenInitialized = true;
	}
#endif

	Super::PostInitProperties();
}

void UActorSequence::BindPossessableObject(const FGuid& ObjectId, UObject& PossessedObject, UObject* Context)
{
	AActor* ActorContext = CastChecked<AActor>(Context);

	if (UActorComponent* Component = Cast<UActorComponent>(&PossessedObject))
	{
		ObjectReferences.CreateBinding(ObjectId, FActorSequenceObjectReference::CreateForComponent(Component));
	}
	else if (AActor* Actor = Cast<AActor>(&PossessedObject))
	{
		ObjectReferences.CreateBinding(ObjectId, FActorSequenceObjectReference::CreateForActor(Actor, ActorContext));
	}
}

bool UActorSequence::CanPossessObject(UObject& Object, UObject* InPlaybackContext) const
{
	if (InPlaybackContext == nullptr)
	{
		return false;
	}

	AActor* ActorContext = CastChecked<AActor>(InPlaybackContext);

	if (AActor* Actor = Cast<AActor>(&Object))
	{
		return Actor == InPlaybackContext || Actor->GetLevel() == ActorContext->GetLevel();
	}
	else if (UActorComponent* Component = Cast<UActorComponent>(&Object))
	{
		return Component->GetOwner() ? Component->GetOwner()->GetLevel() == ActorContext->GetLevel() : false;
	}
	return false;
}

void UActorSequence::LocateBoundObjects(const FGuid& ObjectId, UObject* Context, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	if (Context)
	{
		ObjectReferences.ResolveBinding(ObjectId, CastChecked<AActor>(Context), OutObjects);
	}
}

UMovieScene* UActorSequence::GetMovieScene() const
{
	return MovieScene;
}

UObject* UActorSequence::GetParentObject(UObject* Object) const
{
	if (UActorComponent* Component = Cast<UActorComponent>(Object))
	{
		return Component->GetOwner();
	}

	return nullptr;
}

void UActorSequence::UnbindPossessableObjects(const FGuid& ObjectId)
{
	ObjectReferences.RemoveBinding(ObjectId);
}

UObject* UActorSequence::CreateDirectorInstance(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID SequenceID)
{
	AActor* Actor = CastChecked<AActor>(SharedPlaybackState->GetPlaybackContext(), ECastCheckedType::NullAllowed);
	if (!Actor)
	{
		return nullptr;
	}

	// If this sequence is inside a blueprint, or its component's archetype is from a blueprint, we use the actor as the instace (which will be an instance of the blueprint itself)
	if (GetTypedOuter<UBlueprintGeneratedClass>() || GetTypedOuter<UActorSequenceComponent>()->GetArchetype() != GetDefault<UActorSequenceComponent>())
	{
		return Actor;
	}

	// Otherwise we use the level script actor as the instance
	return Actor->GetLevel()->GetLevelScriptActor();
}

#if WITH_EDITOR
FText UActorSequence::GetDisplayName() const
{
	UActorSequenceComponent* Component = GetTypedOuter<UActorSequenceComponent>();

	if (Component)
	{
		FString OwnerName;
		
		if (UBlueprint* Blueprint = GetParentBlueprint())
		{
			OwnerName = Blueprint->GetName();
		}
		else if(AActor* Owner = Component->GetOwner())
		{
			OwnerName = Owner->GetActorLabel();
		}

		return OwnerName.IsEmpty()
			? FText::FromName(Component->GetFName())
			: FText::Format(NSLOCTEXT("ActorSequence", "DisplayName", "{0} ({1})"), FText::FromName(Component->GetFName()), FText::FromString(OwnerName));
	}

	return UMovieSceneSequence::GetDisplayName();
}


ETrackSupport UActorSequence::IsTrackSupportedImpl(TSubclassOf<class UMovieSceneTrack> InTrackClass) const
{
	if (InTrackClass == UMovieSceneAudioTrack::StaticClass() ||
		InTrackClass == UMovieSceneEventTrack::StaticClass() ||
		InTrackClass == UMovieSceneMaterialParameterCollectionTrack::StaticClass() ||
		InTrackClass == UMovieSceneSkeletalAnimationTrack::StaticClass() ||
		InTrackClass == UMovieSceneTimeWarpTrack::StaticClass())
	{
		return ETrackSupport::Supported;
	}

	return Super::IsTrackSupportedImpl(InTrackClass);
}

#endif
