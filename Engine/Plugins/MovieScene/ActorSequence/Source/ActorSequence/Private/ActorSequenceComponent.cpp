// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorSequenceComponent.h"
#include "ActorSequence.h"
#include "ActorSequencePlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorSequenceComponent)

UActorSequenceComponent::UActorSequenceComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;

	if (HasAnyFlags(RF_ClassDefaultObject) || GetArchetype() == GetDefault<UActorSequenceComponent>())
	{
		Sequence = ObjectInitializer.CreateDefaultSubobject<UActorSequence>(this, "Sequence");
		Sequence->SetFlags(RF_Public | RF_Transactional);
	}
}

void UActorSequenceComponent::PlaySequence()
{
	if (SequencePlayer != nullptr)
	{
		SequencePlayer->Play();
	}
}

void UActorSequenceComponent::PlaySequenceReverse()
{
	if (SequencePlayer != nullptr)
	{
		SequencePlayer->PlayReverse();
	}
}

void UActorSequenceComponent::PauseSequence()
{
	if (SequencePlayer != nullptr)
	{
		SequencePlayer->Pause();
	}
}

void UActorSequenceComponent::StopSequence()
{
	if (SequencePlayer != nullptr)
	{
		SequencePlayer->Stop();
	}
}

void UActorSequenceComponent::PostInitProperties()
{
	Super::PostInitProperties();
}

void UActorSequenceComponent::BeginPlay()
{
	Super::BeginPlay();

	if (Sequence != nullptr)
	{
		SequencePlayer = NewObject<UActorSequencePlayer>(this, "SequencePlayer");

		SequencePlayer->SetPlaybackClient(this);

		// Initialize this player for tick as soon as possible to ensure that a persistent
		// reference to the tick manager is maintained
		SequencePlayer->InitializeForTick(this);

		SequencePlayer->Initialize(Sequence, PlaybackSettings);

		if (PlaybackSettings.bAutoPlay)
		{
			SequencePlayer->Play();
		}
	}
}

void UActorSequenceComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	if (SequencePlayer)
	{
		// Stop the internal sequence player during EndPlay when it's safer
		// to modify other actor's state during state restoration.
		SequencePlayer->Stop();
		SequencePlayer->TearDown();
	}
	
	Super::EndPlay(EndPlayReason);
}

void UActorSequenceComponent::TickComponent(float DeltaSeconds, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaSeconds, TickType, ThisTickFunction);
}

void UActorSequenceComponent::PostLoad()
{
	Super::PostLoad();

}

