// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorSequenceObjectReference.h"
#include "Engine/SCS_Node.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorSequenceObjectReference)

FActorSequenceObjectReference FActorSequenceObjectReference::CreateForComponent(UActorComponent* InComponent)
{
	check(InComponent);

	FActorSequenceObjectReference NewReference;
	NewReference.Type = EActorSequenceObjectReferenceType::Component;

	AActor* Actor = InComponent->GetOwner();
	if (Actor)
	{
		NewReference.PathToComponent = InComponent->GetPathName(Actor);
		return NewReference;
	}

#if WITH_EDITORONLY_DATA
	UBlueprintGeneratedClass* GeneratedClass = InComponent->GetTypedOuter<UBlueprintGeneratedClass>();
	if (GeneratedClass && GeneratedClass->SimpleConstructionScript)
	{
		UBlueprint* Blueprint = Cast<UBlueprint>(GeneratedClass->ClassGeneratedBy);
		if (Blueprint)
		{
			for (USCS_Node* Node : GeneratedClass->SimpleConstructionScript->GetAllNodes())
			{
				if (Node->ComponentTemplate == InComponent)
				{
					NewReference.PathToComponent = Node->GetVariableName().ToString();
					return NewReference;
				}
			}
		}
	}
#endif

	ensureMsgf(false, TEXT("Unable to find parent actor for component. Reference will be unresolvable."));
	return NewReference;
}

FActorSequenceObjectReference FActorSequenceObjectReference::CreateForActor(AActor* InActor, AActor* ContextActor)
{
	if (InActor == ContextActor)
	{
		return CreateForContextActor();
	}

	FActorSequenceObjectReference NewReference;
	check(InActor && ContextActor && InActor->GetLevel() == ContextActor->GetLevel());

	NewReference.Type = EActorSequenceObjectReferenceType::ExternalActor;
	NewReference.ActorId = FLazyObjectPtr(InActor).GetUniqueID().GetGuid();
	return NewReference;
}

FActorSequenceObjectReference FActorSequenceObjectReference::CreateForContextActor()
{
	FActorSequenceObjectReference NewReference;
	NewReference.Type = EActorSequenceObjectReferenceType::ContextActor;
	return NewReference;
}

UObject* FActorSequenceObjectReference::Resolve(AActor* SourceActor) const
{
	check(SourceActor);

	switch(Type)
	{
	case EActorSequenceObjectReferenceType::ContextActor:
		return SourceActor;

	case EActorSequenceObjectReferenceType::ExternalActor:
		if (ActorId.IsValid())
		{
			// Fixup for PIE
			int32 PIEInstanceID = SourceActor->GetOutermost()->GetPIEInstanceID();
			FUniqueObjectGuid FixedUpId(ActorId);
			if (PIEInstanceID != -1)
			{
				FixedUpId = FixedUpId.FixupForPIE(PIEInstanceID);
			}
			
			FLazyObjectPtr LazyPtr;
			LazyPtr = FixedUpId;

			if (AActor* FoundActor = Cast<AActor>(LazyPtr.Get()))
			{
				if (FoundActor->GetLevel() == SourceActor->GetLevel())
				{
					return FoundActor;
				}
			}
		}
		break;

	case EActorSequenceObjectReferenceType::Component:
		if (!PathToComponent.IsEmpty())
		{
			return FindObject<UActorComponent>(SourceActor, *PathToComponent);
		}
		break;
	}

	return nullptr;
}

bool FActorSequenceObjectReferenceMap::HasBinding(const FGuid& ObjectId) const
{
	return BindingIds.Contains(ObjectId);
}

void FActorSequenceObjectReferenceMap::RemoveBinding(const FGuid& ObjectId)
{
	int32 Index = BindingIds.IndexOfByKey(ObjectId);
	if (Index != INDEX_NONE)
	{
		BindingIds.RemoveAtSwap(Index, EAllowShrinking::No);
		References.RemoveAtSwap(Index, EAllowShrinking::No);
	}
}

void FActorSequenceObjectReferenceMap::CreateBinding(const FGuid& ObjectId, const FActorSequenceObjectReference& ObjectReference)
{
	int32 ExistingIndex = BindingIds.IndexOfByKey(ObjectId);
	if (ExistingIndex == INDEX_NONE)
	{
		ExistingIndex = BindingIds.Num();

		BindingIds.Add(ObjectId);
		References.AddDefaulted();
	}

	References[ExistingIndex].Array.AddUnique(ObjectReference);
}

void FActorSequenceObjectReferenceMap::ResolveBinding(const FGuid& ObjectId, AActor* SourceActor, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	int32 Index = BindingIds.IndexOfByKey(ObjectId);
	if (Index == INDEX_NONE)
	{
		return;
	}

	for (const FActorSequenceObjectReference& Reference : References[Index].Array)
	{
		if (UObject* Object = Reference.Resolve(SourceActor))
		{
			OutObjects.Add(Object);
		}
	}
}
