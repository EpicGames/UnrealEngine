// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorSequencePlayer.h"
#include "ActorSequence.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/SimpleConstructionScript.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorSequencePlayer)


UObject* UActorSequencePlayer::GetPlaybackContext() const
{
	UActorSequence* ActorSequence = CastChecked<UActorSequence>(Sequence);
	if (ActorSequence)
	{
		if (AActor* Actor = ActorSequence->GetTypedOuter<AActor>())
		{
			return Actor;
		}
#if WITH_EDITOR
		else if (UBlueprintGeneratedClass* GeneratedClass = ActorSequence->GetTypedOuter<UBlueprintGeneratedClass>())
		{
			return GeneratedClass->SimpleConstructionScript->GetComponentEditorActorInstance();
		}
#endif
	}

	return nullptr;
}

TArray<UObject*> UActorSequencePlayer::GetEventContexts() const
{
	TArray<UObject*> Contexts;
	if (UObject* PlaybackContext = GetPlaybackContext())
	{
		Contexts.Add(PlaybackContext);
	}
	return Contexts;
}
