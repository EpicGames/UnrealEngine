// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "ActorSequenceObjectReference.h"
#include "ActorSequence.generated.h"

#define UE_API ACTORSEQUENCE_API

/**
 * Movie scene animation embedded within an actor.
 */
UCLASS(MinimalAPI, BlueprintType, Experimental, DefaultToInstanced)
class UActorSequence
	: public UMovieSceneSequence
{
public:
	GENERATED_BODY()

	UE_API UActorSequence(const FObjectInitializer& ObjectInitializer);

	//~ UMovieSceneSequence interface
	UE_API virtual void BindPossessableObject(const FGuid& ObjectId, UObject& PossessedObject, UObject* Context) override;
	UE_API virtual bool CanPossessObject(UObject& Object, UObject* InPlaybackContext) const override;
	UE_API virtual void LocateBoundObjects(const FGuid& ObjectId, UObject* Context, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const override;
	UE_API virtual UMovieScene* GetMovieScene() const override;
	UE_API virtual UObject* GetParentObject(UObject* Object) const override;
	UE_API virtual void UnbindPossessableObjects(const FGuid& ObjectId) override;
	virtual void UnbindObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject* Context) override {}
	virtual void UnbindInvalidObjects(const FGuid& ObjectId, UObject* Context) override {}
	UE_API virtual UObject* CreateDirectorInstance(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID SequenceID) override;

#if WITH_EDITOR
	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual ETrackSupport IsTrackSupportedImpl(TSubclassOf<class UMovieSceneTrack> InTrackClass) const override;
#endif

#if WITH_EDITORONLY_DATA
	UE_API UBlueprint* GetParentBlueprint() const;
#endif

	UE_API bool IsEditable() const;
	
private:

	//~ UObject interface
	UE_API virtual void PostInitProperties() override;

private:
	
	/** Pointer to the movie scene that controls this animation. */
	UPROPERTY(Instanced)
	TObjectPtr<UMovieScene> MovieScene;

	/** Collection of object references. */
	UPROPERTY()
	FActorSequenceObjectReferenceMap ObjectReferences;

#if WITH_EDITOR
public:

	/** Event that is fired to initialize default state for a sequence */
	DECLARE_EVENT_OneParam(UActorSequence, FOnInitialize, UActorSequence*)
	static FOnInitialize& OnInitializeSequence() { return OnInitializeSequenceEvent; }

private:
	static UE_API FOnInitialize OnInitializeSequenceEvent;
#endif

#if WITH_EDITORONLY_DATA
private:
	UPROPERTY()
	bool bHasBeenInitialized;
#endif
};

#undef UE_API
