// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "MovieSceneSequencePlayer.h"
#include "ActorSequenceComponent.generated.h"

#define UE_API ACTORSEQUENCE_API


class UActorSequence;
class UActorSequencePlayer;


/**
 * Movie scene animation embedded within an actor.
 */
UCLASS(MinimalAPI, Blueprintable, Experimental, ClassGroup=Sequence, hidecategories=(Collision, Cooking, Activation), meta=(BlueprintSpawnableComponent))
class UActorSequenceComponent
	: public UActorComponent
{
public:
	GENERATED_BODY()

	UE_API UActorSequenceComponent(const FObjectInitializer& Init);

	UActorSequence* GetSequence() const
	{
		return Sequence;
	}

	UActorSequencePlayer* GetSequencePlayer() const 
	{
		return SequencePlayer;
	}

	/** Calls the Play function on the SequencePlayer if its valid. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API void PlaySequence();

	/** Calls the PlayReverse function on the SequencePlayer if its valid. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API void PlaySequenceReverse();

	/** Calls the Pause function on the SequencePlayer if its valid. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API void PauseSequence();

	/** Calls the Stop function on the SequencePlayer if its valid. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Player")
	UE_API void StopSequence();
	
	// UActorComponent interface
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;

	// UObject interface
	UE_API virtual void PostLoad() override;

protected:

	UPROPERTY(EditAnywhere, Category="Playback", meta=(ShowOnlyInnerProperties))
	FMovieSceneSequencePlaybackSettings PlaybackSettings;

	/** Embedded actor sequence data */
	UPROPERTY(EditAnywhere, Instanced, Category=Animation)
	TObjectPtr<UActorSequence> Sequence;

	UPROPERTY(transient, BlueprintReadOnly, Category=Animation)
	TObjectPtr<UActorSequencePlayer> SequencePlayer;
};

#undef UE_API
