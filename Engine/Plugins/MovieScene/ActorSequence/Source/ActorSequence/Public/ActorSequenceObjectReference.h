// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/LazyObjectPtr.h"
#include "ActorSequenceObjectReference.generated.h"

class UActorComponent;

UENUM()
enum class EActorSequenceObjectReferenceType : uint8
{
	/** The reference relates to the context actor */
	ContextActor,
	/** The reference relates to an actor outside of the context actor actor */
	ExternalActor,
	/** The reference relates to a component */
	Component,
};

/**
 * An external reference to an level sequence object, resolvable through an arbitrary context.
 */
USTRUCT()
struct FActorSequenceObjectReference
{
	GENERATED_BODY()

	/**
	 * Default construction to a null reference
	 */
	FActorSequenceObjectReference()
		: Type(EActorSequenceObjectReferenceType::ContextActor)
	{}

	/**
	 * Generates a new reference to an object within a given context.
	 *
	 * @param InComponent The component to create a reference for
	 */
	ACTORSEQUENCE_API static FActorSequenceObjectReference CreateForComponent(UActorComponent* InComponent);

	/**
	 * Generates a new reference to an object within a given context.
	 *
	 * @param InActor The actor to create a reference for
	 */
	ACTORSEQUENCE_API static FActorSequenceObjectReference CreateForActor(AActor* InActor, AActor* ResolutionContext);

	/**
	 * Generates a new reference to the root actor.
	 */
	ACTORSEQUENCE_API static FActorSequenceObjectReference CreateForContextActor();

	/**
	 * Check whether this object reference is valid or not
	 */
	bool IsValid() const
	{
		return ActorId.IsValid() || !PathToComponent.IsEmpty();
	}

	/**
	 * Resolve this reference from the specified source actor
	 *
	 * @return The object
	 */
	ACTORSEQUENCE_API UObject* Resolve(AActor* SourceActor) const;

	/**
	 * Equality comparator
	 */
	friend bool operator==(const FActorSequenceObjectReference& A, const FActorSequenceObjectReference& B)
	{
		return A.ActorId == B.ActorId && A.PathToComponent == B.PathToComponent;
	}

private:

	/** The type of the binding */
	UPROPERTY()
	EActorSequenceObjectReferenceType Type;

	/** The ID of the actor - if this is set, we're either the owner actor, or an object outside of the context */
	UPROPERTY()
	FGuid ActorId;

	/** Path to the component from its owner actor */
	UPROPERTY()
	FString PathToComponent;
};

USTRUCT()
struct FActorSequenceObjectReferences
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FActorSequenceObjectReference> Array;
};

USTRUCT()
struct FActorSequenceObjectReferenceMap
{
	GENERATED_BODY()

	/**
	 * Check whether this map has a binding for the specified object id
	 * @return true if this map contains a binding for the id, false otherwise
	 */
	bool HasBinding(const FGuid& ObjectId) const;

	/**
	 * Remove a binding for the specified ID
	 *
	 * @param ObjectId	The ID to remove
	 */
	void RemoveBinding(const FGuid& ObjectId);

	/**
	 * Create a binding for the specified ID
	 *
	 * @param ObjectId				The ID to associate the component with
	 * @param ObjectReference	The component reference to bind
	 */
	void CreateBinding(const FGuid& ObjectId, const FActorSequenceObjectReference& ObjectReference);

	/**
	 * Resolve a binding for the specified ID using a given context
	 *
	 * @param ObjectId		The ID to associate the object with
	 * @param ParentActor	The parent actor to resolve within
	 * @param OutObjects	Container to populate with bound components
	 */
	void ResolveBinding(const FGuid& ObjectId, AActor* ParentActor, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const;

private:
	
	UPROPERTY()
	TArray<FGuid> BindingIds;

	UPROPERTY()
	TArray<FActorSequenceObjectReferences> References;
};
