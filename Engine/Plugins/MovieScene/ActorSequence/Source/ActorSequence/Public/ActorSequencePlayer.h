// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMovieScenePlayer.h"
#include "ActorSequence.h"
#include "MovieSceneSequencePlayer.h"
#include "ActorSequencePlayer.generated.h"

#define UE_API ACTORSEQUENCE_API

/**
 * UActorSequencePlayer is used to actually "play" an actor sequence asset at runtime.
 */
UCLASS(MinimalAPI, BlueprintType)
class UActorSequencePlayer
	: public UMovieSceneSequencePlayer
{
public:
	GENERATED_BODY()

protected:

	//~ IMovieScenePlayer interface
	UE_API virtual UObject* GetPlaybackContext() const override;
	UE_API virtual TArray<UObject*> GetEventContexts() const override;
};

#undef UE_API
