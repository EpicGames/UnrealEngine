// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ActorSequenceEditor : ModuleRules
{
	public ActorSequenceEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ActorSequence",
				"BlueprintGraph",
				"ClassViewer",
				"Core",
				"CoreUObject",
				"EditorFramework",
				"Engine",
				"InputCore",
				"Kismet",
				"MovieScene",
				"MovieSceneTools",
				"Sequencer",
				"Slate",
				"SlateCore",
				"SubobjectEditor",
				"SubobjectDataInterface",
				"UnrealEd",
				"TimeManagement",
				"ToolMenus"
			}
		);

		var DynamicModuleNames = new string[] {
			"LevelEditor",
			"PropertyEditor",
		};

		foreach (var Name in DynamicModuleNames)
		{
			PrivateIncludePathModuleNames.Add(Name);
			DynamicallyLoadedModuleNames.Add(Name);
		}
	}
}
