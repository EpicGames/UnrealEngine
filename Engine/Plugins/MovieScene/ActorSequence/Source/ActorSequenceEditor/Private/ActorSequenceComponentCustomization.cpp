// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorSequenceComponentCustomization.h"

#include "ActorSequence.h"
#include "ActorSequenceComponent.h"
#include "IDetailsView.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Docking/SDockTab.h"
#include "ActorSequenceEditorTabSummoner.h"
#include "IPropertyUtilities.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ActorSequenceComponentCustomization"

FName SequenceTabId("EmbeddedSequenceID");

class SActorSequenceEditorWidgetWrapper : public SActorSequenceEditorWidget
{
public:
	~SActorSequenceEditorWidgetWrapper()
	{
		FCoreUObjectDelegates::OnObjectsReplaced.Remove(OnObjectsReplacedHandle);
	}

	void Construct(const FArguments& InArgs, TWeakObjectPtr<UActorSequenceComponent> InSequenceComponent)
	{
		SActorSequenceEditorWidget::Construct(InArgs, nullptr);

		WeakSequenceComponent = InSequenceComponent;
		AssignSequence(GetActorSequence());

		OnObjectsReplacedHandle = FCoreUObjectDelegates::OnObjectsReplaced.AddSP(this, &SActorSequenceEditorWidgetWrapper::OnObjectsReplaced);
	}

protected:

	UActorSequence* GetActorSequence() const
	{
		UActorSequenceComponent* SequenceComponent = WeakSequenceComponent.Get();
		return SequenceComponent ? SequenceComponent->GetSequence() : nullptr;
	}

	void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
	{
		UActorSequenceComponent* Component = WeakSequenceComponent.Get(true);

		UActorSequenceComponent* NewSequenceComponent = Component ? Cast<UActorSequenceComponent>(ReplacementMap.FindRef(Component)) : nullptr;
		if (NewSequenceComponent)
		{
			WeakSequenceComponent = NewSequenceComponent;
			AssignSequence(GetActorSequence());
		}
	}

private:

	TWeakObjectPtr<UActorSequenceComponent> WeakSequenceComponent;
	FDelegateHandle OnObjectsReplacedHandle;
};

TSharedRef<IDetailCustomization> FActorSequenceComponentCustomization::MakeInstance()
{
	return MakeShared<FActorSequenceComponentCustomization>();
}

void FActorSequenceComponentCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	PropertyUtilities = DetailBuilder.GetPropertyUtilities();

	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);
	if (Objects.Num() != 1)
	{
		// @todo: Display other common properties / information?
		return;
	}

	WeakSequenceComponent = Cast<UActorSequenceComponent>(Objects[0].Get());
	if (!WeakSequenceComponent.Get())
	{
		return;
	}

	TSharedPtr<FTabManager> HostTabManager;
	if (TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr())
	{
		HostTabManager = DetailsView->GetHostTabManager();
	}

	DetailBuilder.HideProperty("Sequence");

	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory("Sequence", FText(), ECategoryPriority::Important);

	bool bIsExternalTabAlreadyOpened = false;

	if (HostTabManager.IsValid() && HostTabManager->HasTabSpawner(SequenceTabId))
	{
		WeakTabManager = HostTabManager;

		TSharedPtr<SDockTab> ExistingTab = HostTabManager->FindExistingLiveTab(SequenceTabId);
		if (ExistingTab.IsValid())
		{
			UActorSequence* ThisSequence = GetActorSequence();

			auto SequencerWidget = StaticCastSharedRef<SActorSequenceEditorWidget>(ExistingTab->GetContent());
			bIsExternalTabAlreadyOpened = ThisSequence && SequencerWidget->GetSequence() == ThisSequence;
		}

		Category.AddCustomRow(FText())
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("SequenceValueText", "Sequence"))
				.Font(DetailBuilder.GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SButton)
				.OnClicked(this, &FActorSequenceComponentCustomization::InvokeSequencer)
				[
					SNew(STextBlock)
					.Text(bIsExternalTabAlreadyOpened ? LOCTEXT("FocusSequenceTabButtonText", "Focus Tab") : LOCTEXT("OpenSequenceTabButtonText", "Open in Tab"))
					.Font(DetailBuilder.GetDetailFont())
				]
			];
	}

	// Only display an inline editor for non-blueprint sequences
	if (GetActorSequence() && !GetActorSequence()->GetParentBlueprint() && !bIsExternalTabAlreadyOpened)
	{
		Category.AddCustomRow(FText())
		.WholeRowContent()
		.MaxDesiredWidth(TOptional<float>())
		[
			SAssignNew(InlineSequencer, SBox)
			.HeightOverride(300)
			[
				SNew(SActorSequenceEditorWidgetWrapper, WeakSequenceComponent)
			]
		];
	}
}

FReply FActorSequenceComponentCustomization::InvokeSequencer()
{
	TSharedPtr<FTabManager> TabManager = WeakTabManager.Pin();
	if (TabManager.IsValid() && TabManager->HasTabSpawner(SequenceTabId))
	{
		if (TSharedPtr<SDockTab> Tab = TabManager->TryInvokeTab(SequenceTabId))
		{
			{
				// Set up a delegate that forces a refresh of this panel when the tab is closed to ensure we see the inline widget
				TWeakPtr<IPropertyUtilities> WeakUtilities = PropertyUtilities;
				auto OnClosed = [WeakUtilities](TSharedRef<SDockTab>)
				{
					TSharedPtr<IPropertyUtilities> PinnedPropertyUtilities = WeakUtilities.Pin();
					if (PinnedPropertyUtilities.IsValid())
					{
						PinnedPropertyUtilities->EnqueueDeferredAction(FSimpleDelegate::CreateSP(PinnedPropertyUtilities.ToSharedRef(), &IPropertyUtilities::ForceRefresh));
					}
				};

				Tab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateLambda(OnClosed));
			}

			// Move our inline widget content to the tab (so that we keep the existing sequencer state)
			if (InlineSequencer.IsValid())
			{
				Tab->SetContent(InlineSequencer->GetChildren()->GetChildAt(0));
				InlineSequencer->SetContent(SNullWidget::NullWidget);
				InlineSequencer->SetVisibility(EVisibility::Collapsed);
			}
			else
			{
				StaticCastSharedRef<SActorSequenceEditorWidget>(Tab->GetContent())->AssignSequence(GetActorSequence());
			}
		}
	}

	PropertyUtilities->ForceRefresh();

	return FReply::Handled();
}

UActorSequence* FActorSequenceComponentCustomization::GetActorSequence() const
{
	UActorSequenceComponent* SequenceComponent = WeakSequenceComponent.Get();
	return SequenceComponent ? SequenceComponent->GetSequence() : nullptr;
}

#undef LOCTEXT_NAMESPACE
