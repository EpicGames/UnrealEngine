// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"

class FReply;
class FTabManager;
class SBox;

class IDetailsView;
class UActorSequence;
class UActorSequenceComponent;
class ISequencer;
class FSCSEditorTreeNode;
class IPropertyUtilities;

class FActorSequenceComponentCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:

	FReply InvokeSequencer();
	UActorSequence* GetActorSequence() const;

	TWeakObjectPtr<UActorSequenceComponent> WeakSequenceComponent;
	TWeakPtr<FTabManager> WeakTabManager;
	TSharedPtr<SBox> InlineSequencer;
	TSharedPtr<IPropertyUtilities> PropertyUtilities;
};
