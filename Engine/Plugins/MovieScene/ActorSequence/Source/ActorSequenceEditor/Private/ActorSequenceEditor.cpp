// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintEditorModule.h"
#include "BlueprintEditorTabs.h"
#include "ActorSequenceComponentCustomization.h"
#include "MovieSceneSequenceEditor_ActorSequence.h"
#include "ActorSequenceEditorStyle.h"
#include "ActorSequenceEditorTabSummoner.h"
#include "Framework/Docking/LayoutExtender.h"
#include "LevelEditor.h"
#include "MovieSceneToolsProjectSettings.h"
#include "PropertyEditorModule.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "ISettingsModule.h"
#include "SequencerSettings.h"
#include "ISequencerModule.h"


#define LOCTEXT_NAMESPACE "ActorSequenceEditor"


/** Shared class type that ensures safe binding to RegisterBlueprintEditorTab through an SP binding without interfering with module ownership semantics */
class FActorSequenceEditorTabBinding
	: public TSharedFromThis<FActorSequenceEditorTabBinding>
{
public:

	FActorSequenceEditorTabBinding()
	{
		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
		BlueprintEditorTabSpawnerHandle = BlueprintEditorModule.OnRegisterTabsForEditor().AddRaw(this, &FActorSequenceEditorTabBinding::RegisterBlueprintEditorTab);
		BlueprintEditorLayoutExtensionHandle = BlueprintEditorModule.OnRegisterLayoutExtensions().AddRaw(this, &FActorSequenceEditorTabBinding::RegisterBlueprintEditorLayout);

		FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditorTabSpawnerHandle = LevelEditor.OnRegisterTabs().AddRaw(this, &FActorSequenceEditorTabBinding::RegisterLevelEditorTab);
		LevelEditorLayoutExtensionHandle = LevelEditor.OnRegisterLayoutExtensions().AddRaw(this, &FActorSequenceEditorTabBinding::RegisterLevelEditorLayout);
	}

	void RegisterLevelEditorLayout(FLayoutExtender& Extender)
	{
		Extender.ExtendLayout(FTabId("ContentBrowserTab1"), ELayoutExtensionPosition::Before, FTabManager::FTab(FName("EmbeddedSequenceID"), ETabState::ClosedTab));
	}

	void RegisterBlueprintEditorLayout(FLayoutExtender& Extender)
	{
		Extender.ExtendLayout(FBlueprintEditorTabs::CompilerResultsID, ELayoutExtensionPosition::Before, FTabManager::FTab(FName("EmbeddedSequenceID"), ETabState::ClosedTab));
	}

	void RegisterBlueprintEditorTab(FWorkflowAllowedTabSet& TabFactories, FName InModeName, TSharedPtr<FBlueprintEditor> BlueprintEditor)
	{
		TabFactories.RegisterFactory(MakeShared<FActorSequenceEditorSummoner>(BlueprintEditor));
	}

	void RegisterLevelEditorTab(TSharedPtr<FTabManager> InTabManager)
	{
		auto SpawnTab = [](const FSpawnTabArgs&) -> TSharedRef<SDockTab>
			{
				TSharedRef<SActorSequenceEditorWidget> Widget = SNew(SActorSequenceEditorWidget, nullptr);

				return SNew(SDockTab)
					.Label(&Widget.Get(), &SActorSequenceEditorWidget::GetDisplayLabel)
					[
						Widget
					];
			};

		InTabManager->RegisterTabSpawner("EmbeddedSequenceID", FOnSpawnTab::CreateStatic(SpawnTab))
			.SetMenuType(ETabSpawnerMenuType::Hidden)
			.SetAutoGenerateMenuEntry(false);
	}

	~FActorSequenceEditorTabBinding()
	{
		FBlueprintEditorModule* BlueprintEditorModule = FModuleManager::GetModulePtr<FBlueprintEditorModule>("Kismet");
		if (BlueprintEditorModule)
		{
			BlueprintEditorModule->OnRegisterTabsForEditor().Remove(BlueprintEditorTabSpawnerHandle);
			BlueprintEditorModule->OnRegisterLayoutExtensions().Remove(BlueprintEditorLayoutExtensionHandle);
		}

		FLevelEditorModule* LevelEditor = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor");
		if (LevelEditor)
		{
			LevelEditor->OnRegisterTabs().Remove(LevelEditorTabSpawnerHandle);
			LevelEditor->OnRegisterLayoutExtensions().Remove(LevelEditorLayoutExtensionHandle);
		}
	}

private:

	/** Delegate binding handle for FBlueprintEditorModule::OnRegisterTabsForEditor */
	FDelegateHandle BlueprintEditorTabSpawnerHandle, BlueprintEditorLayoutExtensionHandle;

	FDelegateHandle LevelEditorTabSpawnerHandle, LevelEditorLayoutExtensionHandle;
};

/**
 * Implements the ActorSequenceEditor module.
 */
class FActorSequenceEditorModule
	: public IModuleInterface, public FGCObject
{
public:

	FActorSequenceEditorModule()
		: Settings(nullptr)
	{
	}

	virtual void StartupModule() override
	{
		// Register styles
		FActorSequenceEditorStyle::Get();

		BlueprintEditorTabBinding = MakeShared<FActorSequenceEditorTabBinding>();
		RegisterCustomizations();
		RegisterSettings();
		OnInitializeSequenceHandle = UActorSequence::OnInitializeSequence().AddStatic(FActorSequenceEditorModule::OnInitializeSequence);

		ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
		SequenceEditorHandle = SequencerModule.RegisterSequenceEditor(UActorSequence::StaticClass(), MakeUnique<FMovieSceneSequenceEditor_ActorSequence>());
	}
	
	virtual void ShutdownModule() override
	{
		UActorSequence::OnInitializeSequence().Remove(OnInitializeSequenceHandle);
		UnregisterCustomizations();
		UnregisterSettings();

		ISequencerModule* SequencerModule = FModuleManager::Get().GetModulePtr<ISequencerModule>("Sequencer");
		if (SequencerModule)
		{
			SequencerModule->UnregisterSequenceEditor(SequenceEditorHandle);
		}

		BlueprintEditorTabBinding = nullptr;
	}

	static void OnInitializeSequence(UActorSequence* Sequence)
	{
		auto* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();
		UMovieScene* MovieScene = Sequence->GetMovieScene();
		
		FFrameNumber StartFrame = (ProjectSettings->DefaultStartTime * MovieScene->GetTickResolution()).RoundToFrame();
		int32        Duration   = (ProjectSettings->DefaultDuration * MovieScene->GetTickResolution()).RoundToFrame().Value;

		MovieScene->SetPlaybackRange(StartFrame, Duration);
	}

	/** Register details view customizations. */
	void RegisterCustomizations()
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		ActorSequenceComponentName = UActorSequenceComponent::StaticClass()->GetFName();
		PropertyModule.RegisterCustomClassLayout("ActorSequenceComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FActorSequenceComponentCustomization::MakeInstance));
	}

	/** Unregister details view customizations. */
	void UnregisterCustomizations()
	{
		FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
		if (PropertyModule)
		{
			PropertyModule->UnregisterCustomPropertyTypeLayout(ActorSequenceComponentName);
		}
	}

	/** Register settings objects. */
	void RegisterSettings()
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			Settings = USequencerSettingsContainer::GetOrCreate<USequencerSettings>(TEXT("EmbeddedActorSequenceEditor"));

			SettingsModule->RegisterSettings("Editor", "ContentEditors", "EmbeddedActorSequenceEditor",
				LOCTEXT("EmbeddedActorSequenceEditorSettingsName", "Embedded Actor Sequence Editor"),
				LOCTEXT("EmbeddedActorSequenceEditorSettingsDescription", "Configure the look and feel of the Embedded Actor Sequence Editor."),
				Settings);	
		}
	}

	/** Unregister settings objects. */
	void UnregisterSettings()
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Editor", "ContentEditors", "EmbeddedActorSequenceEditor");
		}
	}

	/** FGCObject interface */
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
	{
		if (Settings)
		{
			Collector.AddReferencedObject(Settings);
		}
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FActorSequenceEditorModule");
	}

	FDelegateHandle SequenceEditorHandle;
	FDelegateHandle OnInitializeSequenceHandle;
	TSharedPtr<FActorSequenceEditorTabBinding> BlueprintEditorTabBinding;
	FName ActorSequenceComponentName;
	TObjectPtr<USequencerSettings> Settings;
};

IMPLEMENT_MODULE(FActorSequenceEditorModule, ActorSequenceEditor);

#undef LOCTEXT_NAMESPACE
