// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyle.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleRegistry.h"


class FActorSequenceEditorStyle final
	: public FSlateStyleSet
{
public:
	FActorSequenceEditorStyle()
		: FSlateStyleSet("ActorSequenceEditorStyle")
	{
		const FVector2D Icon16x16(16.0f, 16.0f);
		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("MovieScene/ActorSequence/Content"));

		Set("ClassIcon.ActorSequence", new FSlateImageBrush(RootToContentDir(TEXT("ActorSequence_16x.png")), Icon16x16));
		Set("ClassIcon.ActorSequenceComponent", new FSlateImageBrush(RootToContentDir(TEXT("ActorSequence_16x.png")), Icon16x16));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	static FActorSequenceEditorStyle& Get()
	{
		static FActorSequenceEditorStyle Inst;
		return Inst;
	}
	
	~FActorSequenceEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
};
