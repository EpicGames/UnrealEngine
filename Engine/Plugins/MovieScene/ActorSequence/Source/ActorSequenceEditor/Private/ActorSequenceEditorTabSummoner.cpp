// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorSequenceEditorTabSummoner.h"

#include "ActorSequence.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "ISequencerModule.h"
#include "Engine/SimpleConstructionScript.h"
#include "LevelEditorSequencerIntegration.h"
#include "Framework/Views/TableViewMetadata.h"
#include "SSCSEditor.h"
#include "MovieScenePossessable.h"
#include "Styling/SlateIconFinder.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "LevelEditor.h"
#include "SubobjectData.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "ActorSequenceEditorStyle.h"
#include "EventHandlers/ISignedObjectEventHandler.h"
#include "Toolkits/IToolkitHost.h"
#include "UObject/ObjectSaveContext.h"
#include "SSubobjectEditor.h"
#include "EditorViewportSelectability.h"
#include "EditorViewportSelectabilityBridge.h"
#include "SequencerCommands.h"

#define LOCTEXT_NAMESPACE "ActorSequenceEditorSummoner"

DECLARE_DELEGATE_OneParam(FOnComponentSelected, TSharedPtr<FSCSEditorTreeNode>);
DECLARE_DELEGATE_RetVal_OneParam(bool, FIsComponentValid, UActorComponent*);

namespace UE::ActorSequenceEditor::Private
{
	const FName ViewportToolbarOwnerName = "ActorSequenceEditorViewportToolbar";

	FToolMenuEntry CreateToggleViewportSelectionEntry(TWeakPtr<ISequencer> InWeakSequencer)
	{
		return FToolMenuEntry::InitDynamicEntry(
			"DynamicToggleViewportSelection",
			FNewToolMenuSectionDelegate::CreateLambda(
				[InWeakSequencer](FToolMenuSection& InDynamicSection)
				{
					// First attempt to use the non-context Sequencer.
					TSharedPtr<ISequencer> Sequencer = InWeakSequencer.Pin();
					if (!Sequencer)
					{
						return;
					}

					if (!Sequencer->GetHostCapabilities().bSupportsViewportSelectability)
					{
						return;
					}

					const TSharedPtr<FUICommandList> CommandList =
						Sequencer->GetCommandBindings(ESequencerCommandBindings::Sequencer);

					FToolMenuEntry& Entry = InDynamicSection.AddMenuEntryWithCommandList(
						FSequencerCommands::Get().ToggleLimitViewportSelection, CommandList
					);
					Entry.InsertPosition.Position = EToolMenuInsertType::Last;
					Entry.SetShowInToolbarTopLevel(true);
				}
			)
		);
	}

}; // namespace UE::ActorSequenceEditor::Private

class SComponentSelectionTree
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SComponentSelectionTree) : _IsInEditMode(false) {}

		SLATE_EVENT(FOnComponentSelected, OnComponentSelected)
		SLATE_EVENT(FIsComponentValid, IsComponentValid)
		SLATE_ARGUMENT(bool, IsInEditMode)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, AActor* InPreviewActor)
	{
		bIsInEditMode = InArgs._IsInEditMode;
		OnComponentSelected = InArgs._OnComponentSelected;
		IsComponentValid = InArgs._IsComponentValid;

		ChildSlot
		[
			SAssignNew(TreeView, STreeView<TSharedPtr<FSCSEditorTreeNode>>)
			.TreeItemsSource(&RootNodes)
			.SelectionMode(ESelectionMode::Single)
			.OnGenerateRow(this, &SComponentSelectionTree::GenerateRow)
			.OnGetChildren(this, &SComponentSelectionTree::OnGetChildNodes)
			.OnSelectionChanged(this, &SComponentSelectionTree::OnSelectionChanged)
		];

		BuildTree(InPreviewActor);

		if (RootNodes.Num() == 0)
		{
			ChildSlot
			[
				SNew(SBox)
				.Padding(FMargin(5.f))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NoValidComponentsFound", "No valid components available"))
				]
			];
		}
	}

	void BuildTree(AActor* Actor)
	{
		RootNodes.Reset();
		ObjectToNode.Reset();

		for (UActorComponent* Component : TInlineComponentArray<UActorComponent*>(Actor))
		{
			if (IsComponentVisibleInTree(Component))
			{
				FindOrAddNodeForComponent(Component);
			}
		}
	}

private:

	void OnSelectionChanged(TSharedPtr<FSCSEditorTreeNode> InNode, ESelectInfo::Type SelectInfo)
	{
		OnComponentSelected.ExecuteIfBound(InNode);
	}

	void OnGetChildNodes(TSharedPtr<FSCSEditorTreeNode> InNodePtr, TArray<TSharedPtr<FSCSEditorTreeNode>>& OutChildren)
	{
		OutChildren = InNodePtr->GetChildren();
	}

	TSharedRef<ITableRow> GenerateRow(TSharedPtr<FSCSEditorTreeNode> InNodePtr, const TSharedRef<STableViewBase>& OwnerTable)
	{
		const FSlateBrush* ComponentIcon = FAppStyle::GetBrush("SCS.NativeComponent");
		if (InNodePtr->GetComponentTemplate() != NULL)
		{
			ComponentIcon = FSlateIconFinder::FindIconBrushForClass( InNodePtr->GetComponentTemplate()->GetClass(), TEXT("SCS.Component") );
		}

		FText Label = InNodePtr->IsInheritedComponent() && !bIsInEditMode
			? FText::Format(LOCTEXT("NativeComponentFormatString","{0} (Inherited)"), FText::FromString(InNodePtr->GetDisplayString()))
			: FText::FromString(InNodePtr->GetDisplayString());

		TSharedRef<STableRow<FSCSEditorTreeNodePtrType>> Row = SNew(STableRow<FSCSEditorTreeNodePtrType>, OwnerTable).Padding(FMargin(0.f, 0.f, 0.f, 4.f));
		Row->SetContent(
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(ComponentIcon)
				.ColorAndOpacity(FSCSEditorTreeNode::GetColorTintForIcon(InNodePtr))
			]

			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(2, 0, 0, 0)
			[
				SNew(STextBlock)
				.Text(Label)
			]);

		return Row;
	}

	bool IsComponentVisibleInTree(UActorComponent* ActorComponent) const
	{
		return !IsComponentValid.IsBound() || IsComponentValid.Execute(ActorComponent);
	}

	TSharedPtr<FSCSEditorTreeNode> FindOrAddNodeForComponent(UActorComponent* ActorComponent)
	{
		if (ActorComponent->IsEditorOnly())
		{
			return nullptr;
		}

		if (TSharedPtr<FSCSEditorTreeNode>* Existing = ObjectToNode.Find(ActorComponent))
		{
			return *Existing;
		}
		else if (USceneComponent* SceneComponent = Cast<USceneComponent>(ActorComponent))
		{
			if (UActorComponent* Parent = SceneComponent->GetAttachParent())
			{
				TSharedPtr<FSCSEditorTreeNode> ParentNode = FindOrAddNodeForComponent(Parent);

				if (!ParentNode.IsValid())
				{
					return nullptr;
				}

				TreeView->SetItemExpansion(ParentNode, true);

				TSharedPtr<FSCSEditorTreeNode> ChildNode = ParentNode->AddChildFromComponent(ActorComponent);
				ObjectToNode.Add(ActorComponent, ChildNode);

				return ChildNode;
			}
		}
		
		TSharedPtr<FSCSEditorTreeNode> RootNode = FSCSEditorTreeNode::FactoryNodeFromComponent(ActorComponent);
		RootNodes.Add(RootNode);
		ObjectToNode.Add(ActorComponent, RootNode);

		TreeView->SetItemExpansion(RootNode, true);

		return RootNode;
	}

private:
	bool bIsInEditMode;
	FOnComponentSelected OnComponentSelected;
	FIsComponentValid IsComponentValid;
	TSharedPtr<STreeView<TSharedPtr<FSCSEditorTreeNode>>> TreeView;
	TMap<FObjectKey, TSharedPtr<FSCSEditorTreeNode>> ObjectToNode;
	TArray<TSharedPtr<FSCSEditorTreeNode>> RootNodes;
};

class SActorSequenceEditorWidgetImpl : public SCompoundWidget, public FEditorUndoClient, UE::MovieScene::TIntrusiveEventHandler<UE::MovieScene::ISignedObjectEventHandler>
{
public:

	SLATE_BEGIN_ARGS(SActorSequenceEditorWidgetImpl){}
	SLATE_END_ARGS();

	void Close()
	{
		if (Sequencer.IsValid())
		{
			if (OnGlobalTimeChangedHandle.IsValid())
			{
				Sequencer->OnGlobalTimeChanged().Remove(OnGlobalTimeChangedHandle);
			}
			if (OnSelectionChangedHandle.IsValid())
			{
				Sequencer->GetSelectionChangedObjectGuids().Remove(OnSelectionChangedHandle);
			}
			if (OnSelectionLimitedChangedHandle.IsValid())
			{
				Sequencer->OnViewportSelectionLimitedChanged().Remove(OnSelectionLimitedChangedHandle);
			}

			// Turn off selection limiting if this tab is closed to avoid user confusion
			Sequencer->SetViewportSelectionLimited(false);

			FLevelEditorSequencerIntegration::Get().RemoveSequencer(Sequencer.ToSharedRef());
			Sequencer->Close();
			Sequencer = nullptr;
		}

		if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
		{
			LevelEditorModule->OnMapChanged().RemoveAll(this);
		}

		if (Content)
		{
			Content->SetContent(SNew(STextBlock).Text(LOCTEXT("NothingSelected", "Select a sequence")));
		}

		if (const TSharedPtr<FBlueprintEditor> BlueprintEditor = WeakBlueprintEditor.Pin())
		{
			if (BlueprintEditor->IsHosted())
			{
				const FName CurveEditorTabName = FName(TEXT("SequencerGraphEditor"));
				TSharedPtr<SDockTab> ExistingTab = WeakBlueprintEditor.Pin()->GetToolkitHost()->GetTabManager()->FindExistingLiveTab(CurveEditorTabName);
				if (ExistingTab)
				{
					ExistingTab->RequestCloseTab();
				}
			}

			if (FEditorViewportSelectabilityBridge* SelectabilityBridge = BlueprintEditor->GetViewportSelectabilityBridge())
			{
				SelectabilityBridge->OnIsViewportSelectionLimited().Unbind();
				SelectabilityBridge->OnGetIsObjectSelectableInViewport().Unbind();
			}

			// Remove the viewport toolbar extensions we added earlier.
			UToolMenus::Get()->UnregisterOwnerByName(UE::ActorSequenceEditor::Private::ViewportToolbarOwnerName);
		}

		GEditor->UnregisterForUndo(this);
		GEditor->OnBlueprintPreCompile().Remove(OnBlueprintPreCompileHandle);
		FCoreUObjectDelegates::OnObjectPreSave.Remove(OnObjectSavedHandle);
	}

	~SActorSequenceEditorWidgetImpl()
	{
		Close();
	}
	
	TSharedRef<SDockTab> SpawnCurveEditorTab(const FSpawnTabArgs&)
	{
		return SNew(SDockTab)
			.Label(NSLOCTEXT("Sequencer", "SequencerMainGraphEditorTitle", "Sequencer Curves"))
			[
				SNullWidget::NullWidget
			];
	}

	void Construct(const FArguments&, TWeakPtr<FBlueprintEditor> InBlueprintEditor)
	{
		OnBlueprintPreCompileHandle = GEditor->OnBlueprintPreCompile().AddSP(this, &SActorSequenceEditorWidgetImpl::OnBlueprintPreCompile);
		OnObjectSavedHandle = FCoreUObjectDelegates::OnObjectPreSave.AddSP(this, &SActorSequenceEditorWidgetImpl::OnObjectPreSave);

		WeakBlueprintEditor = InBlueprintEditor;

		{
			const FName CurveEditorTabName = FName(TEXT("SequencerGraphEditor"));
			const FSlateIcon SequencerGraphIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCurveEditor.TabIcon");
			if (WeakBlueprintEditor.IsValid() && !WeakBlueprintEditor.Pin()->GetTabManager()->HasTabSpawner(CurveEditorTabName))
			{
				// Register an empty tab to spawn the Curve Editor in so that layouts restore properly.
				WeakBlueprintEditor.Pin()->GetTabManager()->RegisterTabSpawner(CurveEditorTabName,
					FOnSpawnTab::CreateSP(this, &SActorSequenceEditorWidgetImpl::SpawnCurveEditorTab))
					.SetMenuType(ETabSpawnerMenuType::Type::Hidden)
					.SetIcon(SequencerGraphIcon);
			}
		}

		ChildSlot
		[
			SAssignNew(Content, SBox)
			.MinDesiredHeight(200)
		];

		GEditor->RegisterForUndo(this);
	}


	virtual void PostUndo(bool bSuccess) override
	{
		if (!GetActorSequence())
		{
			Close();
		}
	}

	FText GetDisplayLabel() const
	{
		UActorSequence* Sequence = WeakSequence.Get();
		return Sequence ? Sequence->GetDisplayName() : LOCTEXT("DefaultSequencerLabel", "Sequencer");
	}

	UActorSequence* GetActorSequence() const
	{
		return WeakSequence.Get();
	}

	UObject* GetPlaybackContext() const
	{
		UActorSequence* LocalActorSequence = GetActorSequence();
		if (LocalActorSequence)
		{
			if (AActor* Actor = LocalActorSequence->GetTypedOuter<AActor>())
			{
				return Actor;
			}
			else if (UBlueprintGeneratedClass* GeneratedClass = LocalActorSequence->GetTypedOuter<UBlueprintGeneratedClass>())
			{
				if (GeneratedClass->SimpleConstructionScript)
				{
					return GeneratedClass->SimpleConstructionScript->GetComponentEditorActorInstance();
				}
			}
		}
		
		return nullptr;
	}

	TArray<UObject*> GetEventContexts() const
	{
		TArray<UObject*> Contexts;
		if (auto* Context = GetPlaybackContext())
		{
			Contexts.Add(Context);
		}
		return Contexts;
	}

	void SetActorSequence(UActorSequence* NewSequence)
	{
		Unlink();

		WeakSequence = NewSequence;

		if (NewSequence)
		{
			NewSequence->EventHandlers.Link(this);
		}

		// If we already have a sequencer open, just assign the sequence
		if (Sequencer.IsValid() && NewSequence)
		{
			if (Sequencer->GetRootMovieSceneSequence() != NewSequence)
			{
				Sequencer->ResetToNewRootSequence(*NewSequence);
			}
			return;
		}

		// If we're setting the sequence to none, destroy sequencer
		if (!NewSequence)
		{
			if (Sequencer.IsValid())
			{
				FLevelEditorSequencerIntegration::Get().RemoveSequencer(Sequencer.ToSharedRef());
				Sequencer->Close();
				Sequencer = nullptr;
			}

			Content->SetContent(SNew(STextBlock).Text(LOCTEXT("NothingSelected", "Select a sequence")));
			return;
		}

		// We need to initialize a new sequencer instance
		FSequencerInitParams SequencerInitParams;
		{
			TWeakObjectPtr<UActorSequence> LocalWeakSequence = NewSequence;

			SequencerInitParams.RootSequence = NewSequence;
			SequencerInitParams.EventContexts = TAttribute<TArray<UObject*>>(this, &SActorSequenceEditorWidgetImpl::GetEventContexts);
			SequencerInitParams.PlaybackContext = TAttribute<UObject*>(this, &SActorSequenceEditorWidgetImpl::GetPlaybackContext);
			
			if (WeakBlueprintEditor.IsValid())
			{
				SequencerInitParams.ToolkitHost = WeakBlueprintEditor.Pin()->GetToolkitHost();
				SequencerInitParams.HostCapabilities.bSupportsCurveEditor = true;
				SequencerInitParams.HostCapabilities.bSupportsSidebar = true;
				SequencerInitParams.HostCapabilities.bSupportsViewportSelectability = true;
			}

			TSharedRef<FExtender> AddMenuExtender = MakeShareable(new FExtender);

			AddMenuExtender->AddMenuExtension("AddTracks", EExtensionHook::Before, nullptr,
				FMenuExtensionDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder){

					MenuBuilder.AddSubMenu(
						LOCTEXT("AddComponent_Label", "Component"),
						LOCTEXT("AddComponent_ToolTip", "Add a binding to one of this actor's components and allow it to be animated by Sequencer"),
						FNewMenuDelegate::CreateRaw(this, &SActorSequenceEditorWidgetImpl::AddPossessComponentMenuExtensions),
						false /*bInOpenSubMenuOnClick*/,
						FSlateIcon()//"LevelSequenceEditorStyle", "LevelSequenceEditor.PossessNewActor")
						);

				})
			);

			SequencerInitParams.ViewParams.bReadOnly = !WeakBlueprintEditor.IsValid() && !NewSequence->IsEditable();
			SequencerInitParams.bEditWithinLevelEditor = false;
			SequencerInitParams.ViewParams.AddMenuExtender = AddMenuExtender;
			SequencerInitParams.ViewParams.UniqueName = "EmbeddedActorSequenceEditor";
			SequencerInitParams.ViewParams.ScrubberStyle = ESequencerScrubberStyle::FrameBlock;
			SequencerInitParams.ViewParams.OnReceivedFocus.BindRaw(this, &SActorSequenceEditorWidgetImpl::OnSequencerReceivedFocus);
		}

		Sequencer = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer").CreateSequencer(SequencerInitParams);
		Content->SetContent(Sequencer->GetSequencerWidget());

		OnGlobalTimeChangedHandle = Sequencer->OnGlobalTimeChanged().AddSP(this, &SActorSequenceEditorWidgetImpl::OnGlobalTimeChanged);
		OnSelectionChangedHandle = Sequencer->GetSelectionChangedObjectGuids().AddSP(this, &SActorSequenceEditorWidgetImpl::OnSelectionChanged);
		OnSelectionLimitedChangedHandle = Sequencer->OnViewportSelectionLimitedChanged().AddSP(this, &SActorSequenceEditorWidgetImpl::OnViewportSelectionLimitedChanged);

		FLevelEditorSequencerIntegrationOptions Options;
		Options.bRequiresLevelEvents = true;
		Options.bRequiresActorEvents = false;
		Options.bForceRefreshDetails = false;

		FLevelEditorSequencerIntegration::Get().AddSequencer(Sequencer.ToSharedRef(), Options);
	
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditorModule.OnMapChanged().AddRaw(this, &SActorSequenceEditorWidgetImpl::HandleMapChanged);

		if (const TSharedPtr<FBlueprintEditor> BlueprintEditor = WeakBlueprintEditor.Pin())
		{
			if (FEditorViewportSelectabilityBridge* SelectabilityBridge = BlueprintEditor->GetViewportSelectabilityBridge())
			{
				SelectabilityBridge->OnIsViewportSelectionLimited().BindSP(this, &SActorSequenceEditorWidgetImpl::IsViewportSelectionLimited);
				SelectabilityBridge->OnGetIsObjectSelectableInViewport().BindSP(this, &SActorSequenceEditorWidgetImpl::IsObjectSelectableInViewport);
			}
		}

		// Extend the SCSEditor viewport toolbar.
		{
			FToolMenuOwnerScoped ScopeOwner(UE::ActorSequenceEditor::Private::ViewportToolbarOwnerName);

			const FName TransformMenu = "SCSEditor.ViewportToolbar.Transform";
			UToolMenu* const Menu = UToolMenus::Get()->ExtendMenu(TransformMenu);

			FToolMenuSection& SelectionSection = Menu->FindOrAddSection("Selection");

			{
				FToolMenuEntry Entry = UE::ActorSequenceEditor::Private::CreateToggleViewportSelectionEntry(Sequencer.ToWeakPtr());
				Entry.InsertPosition.Position = EToolMenuInsertType::Last;

				SelectionSection.AddEntry(Entry);
			}
		}
	}

	void HandleMapChanged(UWorld* NewWorld, EMapChangeType MapChangeType)
	{
		if ((MapChangeType == EMapChangeType::LoadMap || MapChangeType == EMapChangeType::NewMap || MapChangeType == EMapChangeType::TearDownWorld))
		{
			Close();
		}
	}

	void Refresh()
	{
		TSharedPtr<FBlueprintEditor> BlueprintEditor = WeakBlueprintEditor.Pin();
		if (!BlueprintEditor.IsValid() || !BlueprintEditor->GetSubobjectViewport().IsValid() || !BlueprintEditor->GetSubobjectEditor().IsValid())
		{
			return;
		}

		UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
		if (!MovieScene)
		{
			return;
		}

		AActor* PreviewActor = GetPreviewActor();

		bool bNeedsUpdate = false;
		for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
		{
			FMovieScenePossessable& Possessable = MovieScene->GetPossessable(Index);
			for (TWeakObjectPtr<> WeakObject : Sequencer->FindBoundObjects(Possessable.GetGuid(), Sequencer->GetFocusedTemplateID()))
			{
				if (UObject* Object = WeakObject.Get())
				{
					FSubobjectEditorTreeNodePtrType TreeNode = BlueprintEditor->GetSubobjectEditor()->FindSlateNodeForObject(Object);

					if (TreeNode.IsValid())
					{
						const FSubobjectData* Data = TreeNode->GetDataSource();
				
						const USceneComponent* Instance = Cast<USceneComponent>(Data->FindComponentInstanceInActor(PreviewActor));
						USceneComponent* Template = const_cast<USceneComponent*>(Cast<USceneComponent>(Data->GetObject()));

						if (Instance && Template)
						{
							Template->SetRelativeLocation_Direct(Instance->GetRelativeLocation());
							Template->SetRelativeRotation_Direct(Instance->GetRelativeRotation());
							Template->SetRelativeScale3D_Direct(Instance->GetRelativeScale3D());
							bNeedsUpdate = true;
						}
					}
				}
			}
		}

		if (bNeedsUpdate)
		{
			BlueprintEditor->UpdateSubobjectPreview(true);
		}
	}

	void OnSelectionChanged(TArray<FGuid> GuidsChanged)
	{
		Refresh();
	}

	void OnGlobalTimeChanged()
	{
		Refresh();
	}

	void OnSequencerReceivedFocus()
	{
		if (Sequencer.IsValid())
		{
			FLevelEditorSequencerIntegration::Get().OnSequencerReceivedFocus(Sequencer.ToSharedRef());
		}
	}

	void OnObjectPreSave(UObject* InObject, FObjectPreSaveContext SaveContext)
	{
		TSharedPtr<FBlueprintEditor> BlueprintEditor = WeakBlueprintEditor.Pin();
		if (Sequencer.IsValid() && BlueprintEditor.IsValid() && InObject && InObject == BlueprintEditor->GetBlueprintObj())
		{
			Sequencer->RestorePreAnimatedState();
		}
	}

	void OnBlueprintPreCompile(UBlueprint* InBlueprint)
	{
		TSharedPtr<FBlueprintEditor> BlueprintEditor = WeakBlueprintEditor.Pin();
		if (Sequencer.IsValid() && BlueprintEditor.IsValid() && InBlueprint && InBlueprint == BlueprintEditor->GetBlueprintObj())
		{
			Sequencer->RestorePreAnimatedState();
		}
	}

	void OnSelectionUpdated(TSharedPtr<FSCSEditorTreeNode> SelectedNode)
	{
		if (SelectedNode->GetNodeType() != FSCSEditorTreeNode::ComponentNode)
		{
			return;
		}

		UActorComponent* EditingComponent = nullptr;

		TSharedPtr<FBlueprintEditor> BlueprintEditor = WeakBlueprintEditor.Pin();
		if (BlueprintEditor.IsValid())
		{
			UBlueprint* Blueprint = BlueprintEditor->GetBlueprintObj();
			if (Blueprint)
			{
				EditingComponent = SelectedNode->GetOrCreateEditableComponentTemplate(Blueprint);
			}
		}
		else if (AActor* Actor = GetPreviewActor())
		{
			EditingComponent = SelectedNode->FindComponentInstanceInActor(Actor);
		}

		if (EditingComponent)
		{
			const FScopedTransaction Transaction(LOCTEXT("AddComponentToSequencer", "Add component to Sequencer"));
			Sequencer->GetHandleToObject(EditingComponent, true);
		}

		FSlateApplication::Get().DismissAllMenus();
	}

	void AddPossessComponentMenuExtensions(FMenuBuilder& MenuBuilder)
	{
		AActor* Actor = GetPreviewActor();
		if (!Actor)
		{
			return;
		}

		Sequencer->GetEvaluationState()->ClearObjectCaches(*Sequencer);
		TSet<UObject*> AllBoundObjects;

		AllBoundObjects.Add(GetOwnerComponent());

		UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
		for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
		{
			FMovieScenePossessable& Possessable = MovieScene->GetPossessable(Index);
			for (TWeakObjectPtr<> WeakObject : Sequencer->FindBoundObjects(Possessable.GetGuid(), Sequencer->GetFocusedTemplateID()))
			{
				if (UObject* Object = WeakObject.Get())
				{
					AllBoundObjects.Add(Object);
				}
			}
		}

		bool bIdent = false;
		MenuBuilder.AddWidget(
			SNew(SComponentSelectionTree, Actor)
			.IsInEditMode(WeakBlueprintEditor.Pin().IsValid())
			.OnComponentSelected(this, &SActorSequenceEditorWidgetImpl::OnSelectionUpdated)
			.IsComponentValid_Lambda(
				[AllBoundObjects](UActorComponent* Component)
				{
					return !AllBoundObjects.Contains(Component);
				}
			)
			, FText(), !bIdent
		);
	}

	AActor* GetPreviewActor() const
	{
		TSharedPtr<FBlueprintEditor> BlueprintEditor = WeakBlueprintEditor.Pin();
		if (BlueprintEditor.IsValid())
		{
			return BlueprintEditor->GetPreviewActor();
		}
		if (UActorSequence* Sequence = WeakSequence.Get())
		{
			return Sequence->GetTypedOuter<AActor>();
		}
		return nullptr;
	}

	UActorComponent* GetOwnerComponent() const
	{
		UActorSequence* ActorSequence = WeakSequence.Get();
		AActor* Actor = ActorSequence ? GetPreviewActor() : nullptr;

		return Actor ? FindObject<UActorComponent>(Actor, *ActorSequence->GetOuter()->GetName()) : nullptr;
	}

	void OnModifiedIndirectly(UMovieSceneSignedObject*) override
	{
		UActorSequence* ActorSequence = WeakSequence.Get();
		UBlueprint* Blueprint = ActorSequence ? ActorSequence->GetParentBlueprint() : nullptr;

		if (Blueprint)
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
	void OnModifiedDirectly(UMovieSceneSignedObject* Object) override
	{
		OnModifiedIndirectly(Object);
	}

	bool IsViewportSelectionLimited() const
	{
		if (Sequencer.IsValid())
		{
			return Sequencer->IsViewportSelectionLimited();
		}
		return true;
	}

	bool IsObjectSelectableInViewport(UObject* const InObject) const
	{
		if (Sequencer.IsValid())
		{
			return Sequencer->IsObjectSelectableInViewport(InObject);
		}
		return true;
	}

	void OnViewportSelectionLimitedChanged(const bool bInSelectionLimited)
	{
		const TSharedPtr<FBlueprintEditor> BlueprintEditor = WeakBlueprintEditor.Pin();
		if (!BlueprintEditor.IsValid())
		{
			return;
		}

		const TSharedPtr<SSubobjectEditor> SubojectEditor = BlueprintEditor->GetSubobjectEditor();
		if (!SubojectEditor.IsValid())
		{
			return;
		}

		if (FEditorViewportSelectabilityBridge* SelectabilityBridge = BlueprintEditor->GetViewportSelectabilityBridge())
		{
			if (SelectabilityBridge->IsViewportSelectionLimited())
			{
				// Filter out non-selectable nodes from the selection
				TSet<FSubobjectEditorTreeNodePtrType> NewSelectedNodes;
				for (const FSubobjectEditorTreeNodePtrType& Node : SubojectEditor->GetSelectedNodes())
				{
					const FSubobjectData* const Data = Node->GetDataSource();
					const UActorComponent* const Component = Data ? Data->FindComponentInstanceInActor(GetPreviewActor()) : nullptr;
					if (IsValid(Component) && IsObjectSelectableInViewport(const_cast<UActorComponent*>(Component)))
					{
						NewSelectedNodes.Add(Node);
					}
				}

				// Clear selection and select the new nodes
				SubojectEditor->ClearSelection();
				for (const FSubobjectEditorTreeNodePtrType& Node : NewSelectedNodes)
				{
					SubojectEditor->SelectNode(Node, false);
				}
			}
		}
	}

private:
	TWeakObjectPtr<UActorSequence> WeakSequence;

	TWeakPtr<FBlueprintEditor> WeakBlueprintEditor;

	TSharedPtr<SBox> Content;
	TSharedPtr<ISequencer> Sequencer;

	FDelegateHandle OnBlueprintPreCompileHandle;
	FDelegateHandle OnObjectSavedHandle;

	FDelegateHandle OnSelectionChangedHandle;
	FDelegateHandle OnGlobalTimeChangedHandle;
	FDelegateHandle OnSelectionLimitedChangedHandle;
};

void SActorSequenceEditorWidget::Construct(const FArguments&, TWeakPtr<FBlueprintEditor> InBlueprintEditor)
{
	ChildSlot
	[
		SAssignNew(Impl, SActorSequenceEditorWidgetImpl, InBlueprintEditor)
	];
}

FText SActorSequenceEditorWidget::GetDisplayLabel() const
{
	return Impl.Pin()->GetDisplayLabel();
}

void SActorSequenceEditorWidget::AssignSequence(UActorSequence* NewActorSequence)
{
	Impl.Pin()->SetActorSequence(NewActorSequence);
}

UActorSequence* SActorSequenceEditorWidget::GetSequence() const
{
	return Impl.Pin()->GetActorSequence();
}

FActorSequenceEditorSummoner::FActorSequenceEditorSummoner(TSharedPtr<FBlueprintEditor> BlueprintEditor)
	: FWorkflowTabFactory("EmbeddedSequenceID", BlueprintEditor)
	, WeakBlueprintEditor(BlueprintEditor)
{
	bIsSingleton = true;

	TabLabel = LOCTEXT("SequencerTabName", "Sequencer");
	TabIcon = FSlateIcon(FActorSequenceEditorStyle::Get().GetStyleSetName(), "ClassIcon.ActorSequence");
}

TSharedRef<SWidget> FActorSequenceEditorSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SActorSequenceEditorWidget, WeakBlueprintEditor);
}

#undef LOCTEXT_NAMESPACE
