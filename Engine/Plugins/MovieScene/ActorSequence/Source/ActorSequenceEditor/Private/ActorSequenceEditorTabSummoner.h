// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"


class FBlueprintEditor;
class SActorSequenceEditorWidgetImpl;
class UActorSequence;
class UActorSequenceComponent;


class SActorSequenceEditorWidget
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SActorSequenceEditorWidget){}
	SLATE_END_ARGS();

	void Construct(const FArguments&, TWeakPtr<FBlueprintEditor> InBlueprintEditor);
	void AssignSequence(UActorSequence* NewActorSequence);
	UActorSequence* GetSequence() const;
	FText GetDisplayLabel() const;

private:

	TWeakPtr<SActorSequenceEditorWidgetImpl> Impl;
};


struct FActorSequenceEditorSummoner
	: public FWorkflowTabFactory
{
	FActorSequenceEditorSummoner(TSharedPtr<FBlueprintEditor> BlueprintEditor);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

protected:

	TWeakObjectPtr<UActorSequenceComponent> WeakComponent;
	TWeakPtr<FBlueprintEditor> WeakBlueprintEditor;
};
