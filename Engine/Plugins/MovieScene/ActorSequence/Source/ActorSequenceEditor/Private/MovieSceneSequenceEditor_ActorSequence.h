// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorSequenceComponent.h"
#include "GameFramework/Actor.h"
#include "MovieSceneSequenceEditor.h"
#include "ActorSequence.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Engine/LevelScriptBlueprint.h"
#include "Modules/ModuleManager.h"

struct FMovieSceneSequenceEditor_ActorSequence : FMovieSceneSequenceEditor
{
	virtual bool CanCreateEvents(UMovieSceneSequence* InSequence) const
	{
		return true;
	}

	virtual bool CanCreateDirectorBlueprint(UMovieSceneSequence* Sequence) const override
	{
#if WITH_EDITOR
		FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
		const TSharedPtr<IClassViewerFilter>& GlobalClassFilter = ClassViewerModule.GetGlobalClassViewerFilter();
		TSharedRef<FClassViewerFilterFuncs> ClassFilterFuncs = ClassViewerModule.CreateFilterFuncs();
		FClassViewerInitializationOptions ClassViewerOptions = {};

		if (GlobalClassFilter.IsValid())
		{
			return GlobalClassFilter->IsClassAllowed(ClassViewerOptions, UBlueprint::StaticClass(), ClassFilterFuncs);
		}
#endif
		return true;
	}

	virtual UBlueprint* GetBlueprintForSequence(UMovieSceneSequence* InSequence) const override
	{
		UActorSequence* ActorSequence = CastChecked<UActorSequence>(InSequence);
		if (UBlueprint* Blueprint = ActorSequence->GetParentBlueprint())
		{
			return Blueprint;
		}

		UActorSequenceComponent* Component = ActorSequence->GetTypedOuter<UActorSequenceComponent>();
		ULevel* Level = (Component && Component->GetOwner()) ? Component->GetOwner()->GetLevel() : nullptr;

		bool bDontCreateNewBlueprint = true;
		return Level ? Level->GetLevelScriptBlueprint(bDontCreateNewBlueprint) : nullptr;
	}

	virtual UBlueprint* CreateBlueprintForSequence(UMovieSceneSequence* InSequence) const override
	{
		UActorSequence* ActorSequence = CastChecked<UActorSequence>(InSequence);
		check(!ActorSequence->GetParentBlueprint());

		UActorSequenceComponent* Component = ActorSequence->GetTypedOuter<UActorSequenceComponent>();
		ULevel* Level = (Component && Component->GetOwner()) ? Component->GetOwner()->GetLevel() : nullptr;

		bool bDontCreateNewBlueprint = false;
		return Level ? Level->GetLevelScriptBlueprint(bDontCreateNewBlueprint) : nullptr;
	}
};
